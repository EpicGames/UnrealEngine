{
	"FileVersion": 3,
	"Version": 2,
	"VersionName": "1.7",
	"FriendlyName": "Rider Integration",
	"Description": "Allows access to source code in Rider.",
	"Category": "Programming",
	"CreatedBy": "JetBrains",
	"CreatedByURL": "https://www.jetbrains.com/",
	"DocsURL": "https://github.com/JetBrains/RiderSourceCodeAccess",
	"MarketplaceURL" : "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "RiderSourceCodeAccess",
			"Type": "EditorNoCommandlet",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		}
	]
}
