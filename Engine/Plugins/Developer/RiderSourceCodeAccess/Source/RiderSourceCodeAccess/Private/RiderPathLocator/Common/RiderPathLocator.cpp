// Copyright Epic Games, Inc. All Rights Reserved.

#include "RiderPathLocator/RiderPathLocator.h"
#include "Dom/JsonObject.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Regex.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

FString ExtractPathFromSettingsJson(const FString& ToolboxPath)
{
	const FString SettingJsonPath = FPaths::Combine(ToolboxPath, TEXT(".settings.json"));
	if (FPaths::FileExists(SettingJsonPath))
	{
		FString JsonStr;
		FFileHelper::LoadFileToString(JsonStr, *SettingJsonPath);
		TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonStr);
		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
		{
			FString InstallLocation;
			if (JsonObject->TryGetStringField(TEXT("install_location"), InstallLocation))
			{
				if (!InstallLocation.IsEmpty())
				{
					return InstallLocation;
				}
			}
		}
	}
	return {};
}

bool FRiderPathLocator::DirectoryExistsAndNonEmpty(const FString& Path)
{
	return !Path.IsEmpty() && FPaths::DirectoryExists(Path);
}

TArray<FInstallInfo> FRiderPathLocator::GetInstallInfosFromToolbox(const FString& ToolboxPath, const FString& Pattern)
{
	if(!DirectoryExistsAndNonEmpty(ToolboxPath)) return {};
	
	const FString InstallLocationPath = ExtractPathFromSettingsJson(ToolboxPath);
	TArray<FInstallInfo> Result{};
	if(!InstallLocationPath.IsEmpty())
	{
		// Toolbox V1 custom install location search path
		Result = GetInstallInfos(FPaths::Combine(InstallLocationPath, TEXT("apps")), Pattern, FInstallInfo::EInstallType::Toolbox);
		if(Result.Num() != 0) return Result;

		// Toolbox V2 custom install location search path
		return GetInstallInfos(InstallLocationPath, Pattern, FInstallInfo::EInstallType::Toolbox);		
	}

	// Toolbox V1 default install location search path
	Result = GetInstallInfos(FPaths::Combine(ToolboxPath, TEXT("apps")), Pattern, FInstallInfo::EInstallType::Toolbox);
	if(Result.Num() != 0) return Result;

	const FString DefaultInstallLocation = GetDefaultIDEInstallLocationForToolboxV2();
	return GetInstallInfos(DefaultInstallLocation, Pattern, FInstallInfo::EInstallType::Toolbox);
}

FVersion FRiderPathLocator::GetLastBuildVersion(const FString& HistoryJsonPath)
{
	if(!FPaths::FileExists(HistoryJsonPath)) return {};
	
	FString JsonStr;
	FFileHelper::LoadFileToString(JsonStr, *HistoryJsonPath);
	const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonStr);
	TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
	if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) || !JsonObject.IsValid()) return {};
	
	const TArray< TSharedPtr<FJsonValue> >* HistoryField;
	if(!JsonObject->TryGetArrayField(TEXT("history"), HistoryField)) return {};
	if(HistoryField->Num() == 0) return {};

	const TSharedPtr<FJsonObject>* LastItemObject;
	if(!HistoryField->Last()->TryGetObject(LastItemObject)) return {};

	const TSharedPtr<FJsonObject>* ItemObject;
	if(!LastItemObject->Get()->TryGetObjectField(TEXT("item"), ItemObject)) return {};

	FString Build;
	if(!ItemObject->Get()->TryGetStringField(TEXT("build"), Build)) return {};

	return FVersion(Build);
}

FString FRiderPathLocator::GetHistoryJsonPath(const FString& RiderPath)
{
	FString Directory = FPaths::ConvertRelativePathToFull(FPaths::Combine(RiderPath, ".."));
	int8_t SafeCheck = 10;
	while(DirectoryExistsAndNonEmpty(Directory) && SafeCheck-- > 0)
	{
		FString HistoryPath = FPaths::Combine(Directory, ".history.json");
		if(FPaths::FileExists(HistoryPath)) return HistoryPath;

		Directory = FPaths::ConvertRelativePathToFull(FPaths::Combine(Directory, ".."));
	}
	return {};
}

TArray<FInstallInfo> FRiderPathLocator::GetInstallInfos(const FString& ToolboxRiderRootPath, const FString& Pattern, FInstallInfo::EInstallType InstallType)
{
	if(!DirectoryExistsAndNonEmpty(ToolboxRiderRootPath)) return {};
	
	TArray<FInstallInfo> RiderInstallInfos;
	TArray<FString> RiderPaths;
	IFileManager::Get().FindFilesRecursive(RiderPaths, *ToolboxRiderRootPath, *Pattern, true, true);
	for(const FString& RiderPath: RiderPaths)
	{
		TOptional<FInstallInfo> InstallInfo = GetInstallInfoFromRiderPath(RiderPath, InstallType);
		if(!InstallInfo.IsSet()) continue;
		
		FString HistoryJsonPath = GetHistoryJsonPath(RiderPath);
		FVersion Version = GetLastBuildVersion(HistoryJsonPath);
		if(Version.IsInitialized() && InstallInfo->Version != Version) continue;
		
		RiderInstallInfos.Add(InstallInfo.GetValue());
	}
	return RiderInstallInfos;
}

void FRiderPathLocator::ParseProductInfoJson(FInstallInfo& Info, const FString& ProductInfoJsonPath)
{
	FString JsonStr;
	FFileHelper::LoadFileToString(JsonStr, *ProductInfoJsonPath);
	TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonStr);
	TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
	if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
	{
		FString VersionString;
		JsonObject->TryGetStringField(TEXT("buildNumber"), VersionString);
		Info.Version = VersionString;
		if(Info.Version.Major() >= 221)
		{
			Info.SupportUprojectState = FInstallInfo::ESupportUproject::Release;
			return;
		}

		const TArray< TSharedPtr<FJsonValue> >* CustomProperties;
		if(!JsonObject->TryGetArrayField(TEXT("customProperties"), CustomProperties)) return;

		for (const TSharedPtr<FJsonValue>& CustomProperty : *CustomProperties)
		{
			const TSharedPtr<FJsonObject> Item = CustomProperty->AsObject();
			if(!Item.IsValid()) continue;
			
			FString SupportUprojectStateKey;
			const bool bIsValidKey = Item->TryGetStringField(TEXT("key"), SupportUprojectStateKey);
			if(!bIsValidKey) continue;
			if(	!SupportUprojectStateKey.Equals(TEXT("SupportUproject")) &&
				!SupportUprojectStateKey.Equals(TEXT("SupportUprojectState"))) continue;

			FString SupportUprojectStateValue;
			const bool bIsValidValue  = Item->TryGetStringField(TEXT("value"), SupportUprojectStateValue);
			if(!bIsValidValue) continue;
			
			if(SupportUprojectStateValue.Equals(TEXT("Beta"))) Info.SupportUprojectState = FInstallInfo::ESupportUproject::Beta;
			if(SupportUprojectStateValue.Equals(TEXT("Release"))) Info.SupportUprojectState = FInstallInfo::ESupportUproject::Release;
		}
	}
}

TArray<FInstallInfo> FRiderPathLocator::GetInstallInfosFromResourceFile()
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("RiderSourceCodeAccess"));
	if(!Plugin.IsValid()) return {};
	
	const FString RiderLocationsFile = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources"), TEXT("RiderLocations.txt"));
	TArray<FString> RiderLocations;
	if(FFileHelper::LoadFileToStringArray(RiderLocations, *RiderLocationsFile) == false) return {};

	TArray<FInstallInfo> RiderInstallInfos;
	for(const auto& RiderLocation : RiderLocations)
	{
		const FString Location = RiderLocation.TrimStartAndEnd();
		if(Location.StartsWith("#")) continue;
		
		TOptional<FInstallInfo> InstallInfo = GetInstallInfoFromRiderPath(Location, FInstallInfo::EInstallType::Custom);
		if(InstallInfo.IsSet())
		{
			RiderInstallInfos.Add(InstallInfo.GetValue());
		}
	}
	return RiderInstallInfos;
}
