// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"

#if PLATFORM_LINUX

#include "RiderPathLocator/RiderPathLocator.h"

#include "Internationalization/Regex.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"

#include "Runtime/Launch/Resources/Version.h"

FString FRiderPathLocator::GetDefaultIDEInstallLocationForToolboxV2()
{
	// V2 and V1 have the same path on Linux, we don't need to process it extra
	return {};
}

TOptional<FInstallInfo> FRiderPathLocator::GetInstallInfoFromRiderPath(const FString& Path, FInstallInfo::EInstallType InstallType)
{
	if(!FPaths::FileExists(Path))
	{
		return {};
	}
	
	const FString PatternString(TEXT("(.*)(?:\\\\|/)bin"));
	const FRegexPattern Pattern(PatternString);
	FRegexMatcher RiderPathMatcher(Pattern, Path);
	if (!RiderPathMatcher.FindNext())
	{
		return {};
	}

	const FString RiderDir = RiderPathMatcher.GetCaptureGroup(1);
	const FString RiderCppPluginPath = FPaths::Combine(RiderDir, TEXT("plugins"), TEXT("rider-cpp"));
	if (!DirectoryExistsAndNonEmpty(RiderCppPluginPath))
	{
		return {};
	}
	
	FInstallInfo Info;
	Info.Path = Path;
	Info.InstallType = InstallType;
	const FString ProductInfoJsonPath = FPaths::Combine(RiderDir, TEXT("product-info.json"));
	if (FPaths::FileExists(ProductInfoJsonPath))
	{
		ParseProductInfoJson(Info, ProductInfoJsonPath);
	}
	if(!Info.Version.IsInitialized())
	{
		Info.Version = FPaths::GetBaseFilename(RiderDir);
		if(Info.Version.Major() >= 221)
		{
			Info.SupportUprojectState = FInstallInfo::ESupportUproject::Release;
		}
	}
	return Info;
}

static FString GetHomePath()
{
#if ENGINE_MAJOR_VERSION == 4 && ENGINE_MINOR_VERSION <= 20
	TCHAR CHomePath[4096];
	FPlatformMisc::GetEnvironmentVariable(TEXT("HOME"), CHomePath, ARRAY_COUNT(CHomePath));
	const FString FHomePath = CHomePath;
#else
	const FString FHomePath = FPlatformMisc::GetEnvironmentVariable(TEXT("HOME"));
#endif

	return FHomePath;
}

static TArray<FInstallInfo> GetManuallyInstalledRiders()
{
	TArray<FInstallInfo> Result;
	TArray<FString> RiderPaths;

	const TArray RiderLookupPaths = {
		GetHomePath(),
		FString(TEXT("/opt")),
		FPaths::Combine(TEXT("/usr"), TEXT("local"), TEXT("bin"))
	};

	for(const FString& RiderLookupPath: RiderLookupPaths)
	{
		FString RiderLookupPathMask = FPaths::Combine(RiderLookupPath,TEXT("*Rider*"));
		IFileManager::Get().FindFiles(RiderPaths, *RiderLookupPathMask, false, true);

		for(const FString& RiderPath: RiderPaths)
		{
			FString FullPath = FPaths::Combine(RiderLookupPath, RiderPath, TEXT("bin"), TEXT("rider.sh"));
			TOptional<FInstallInfo> InstallInfo = FRiderPathLocator::GetInstallInfoFromRiderPath(FullPath, FInstallInfo::EInstallType::Installed);
			if(InstallInfo.IsSet())
			{
				Result.Add(InstallInfo.GetValue());
			}
		}
		RiderPaths.Empty();
	}

	FString FullPath = TEXT("/snap/rider/current/bin/rider.sh");
	TOptional<FInstallInfo> InstallInfo = FRiderPathLocator::GetInstallInfoFromRiderPath(FullPath, FInstallInfo::EInstallType::Installed);
	if(InstallInfo.IsSet())
	{
		Result.Add(InstallInfo.GetValue());
	}

	return Result;
}

static FString GetToolboxPath()
{
	TArray<FInstallInfo> Result;
	FString LocalAppData = FPaths::Combine(GetHomePath(), TEXT(".local"), TEXT("share"));

	return FPaths::Combine(LocalAppData, TEXT("JetBrains"), TEXT("Toolbox"));
}

static TArray<FInstallInfo> GetInstalledRidersWithLocate()
{
	int32 ReturnCode;
	FString OutResults;
	FString OutErrors;
	if (!FPaths::FileExists(TEXT("/usr/bin/locate")))
	{
		return {};
	}

	FPlatformProcess::ExecProcess(TEXT("/usr/bin/locate"), TEXT("-e bin/rider.sh"), &ReturnCode, &OutResults, &OutErrors);
	if (ReturnCode != 0)
	{
		return {};
	}

	TArray<FString> RiderPaths;
	FString TmpString;
	while(OutResults.Split(TEXT("\n"), &TmpString, &OutResults))
	{
		if(TmpString.Contains(TEXT("snapd")) || TmpString.Contains(TEXT(".local")) || TmpString.Contains(TEXT("/opt")))
		{
			continue;
		}
		RiderPaths.Add(TmpString);
	}

	TArray<FInstallInfo> Result;
	for(const FString& RiderPath: RiderPaths)
	{
		TOptional<FInstallInfo> InstallInfo = FRiderPathLocator::GetInstallInfoFromRiderPath(RiderPath, FInstallInfo::EInstallType::Installed);
		if(InstallInfo.IsSet())
		{
			Result.Add(InstallInfo.GetValue());
		}
	}
	return Result;
}

TSet<FInstallInfo> FRiderPathLocator::CollectAllPaths()
{
	TSet<FInstallInfo> InstallInfos;
	InstallInfos.Append(GetInstalledRidersWithLocate());
	InstallInfos.Append(GetManuallyInstalledRiders());
	InstallInfos.Append(GetInstallInfosFromToolbox(GetToolboxPath(), "Rider.sh"));
	InstallInfos.Append(GetInstallInfosFromResourceFile());
	return InstallInfos;
}
#endif
