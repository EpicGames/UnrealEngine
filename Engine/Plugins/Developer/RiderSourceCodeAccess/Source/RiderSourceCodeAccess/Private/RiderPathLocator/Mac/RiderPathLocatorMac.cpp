// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"

#if PLATFORM_MAC

#include "RiderPathLocator/RiderPathLocator.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"

#include "Runtime/Launch/Resources/Version.h"

TOptional<FInstallInfo> FRiderPathLocator::GetInstallInfoFromRiderPath(const FString& PathToRiderApp, FInstallInfo::EInstallType InstallType)
{
	if(!DirectoryExistsAndNonEmpty(PathToRiderApp))
	{
		return {};
	}

	const FString RiderCppPluginPath = FPaths::Combine(PathToRiderApp, TEXT("Contents"), TEXT("plugins"), TEXT("rider-cpp"));
	if (!DirectoryExistsAndNonEmpty(RiderCppPluginPath))
	{
		return {};
	}

	FInstallInfo Info;
	Info.Path = FPaths::Combine(PathToRiderApp, TEXT("Contents"), TEXT("MacOS"), TEXT("rider"));
	Info.InstallType = InstallType;
	const FString ProductInfoJsonPath = FPaths::Combine(PathToRiderApp, TEXT("Contents"), TEXT("Resources"), TEXT("product-info.json"));
	if (FPaths::FileExists(ProductInfoJsonPath))
	{
		ParseProductInfoJson(Info, ProductInfoJsonPath);
	}
	return Info;
}

static TArray<FInstallInfo> GetManuallyInstalledRiders()
{
	TArray<FInstallInfo> Result;
	TArray<FString> RiderPaths;
	IFileManager::Get().FindFiles(RiderPaths, TEXT("/Applications/Rider*.app"), false, true);
	for(const FString& RiderPath: RiderPaths)
	{
		FString FullPath = TEXT("/Applications/") + RiderPath;
		TOptional<FInstallInfo> InstallInfo = FRiderPathLocator::GetInstallInfoFromRiderPath(FullPath, FInstallInfo::EInstallType::Installed);
		if(InstallInfo.IsSet())
		{
			Result.Add(InstallInfo.GetValue());
		}
	}
	return Result;
}

static FString GetToolboxPath()
{
	TArray<FInstallInfo> Result;
#if ENGINE_MAJOR_VERSION == 4 && ENGINE_MINOR_VERSION <= 20
	TCHAR CHomePath[4096];
	FPlatformMisc::GetEnvironmentVariable(TEXT("HOME"), CHomePath, ARRAY_COUNT(CHomePath));
	const FString FHomePath = CHomePath;
#else
	const FString FHomePath = FPlatformMisc::GetEnvironmentVariable(TEXT("HOME"));
#endif
	FString LocalAppData = FPaths::Combine(FHomePath, TEXT("Library"), TEXT("Application Support"));

	return FPaths::Combine(LocalAppData, TEXT("JetBrains"), TEXT("Toolbox"));
}

static TArray<FInstallInfo> GetInstalledRidersWithMdfind()
{
	int32 ReturnCode;
	FString OutResults;
	FString OutErrors;

	// avoid trying to run mdfind if it doesnt exists
	if (!FPaths::FileExists(TEXT("/usr/bin/mdfind")))
	{
		return {};
	}

	FPlatformProcess::ExecProcess(TEXT("/usr/bin/mdfind"), TEXT("\"kMDItemContentType == com.apple.application-bundle && (kMDItemFSName == *Rider*)\""), &ReturnCode, &OutResults, &OutErrors);
	if (ReturnCode != 0)
	{
		return {};
	}

	TArray<FString> RiderPaths;
	OutResults.ParseIntoArray(RiderPaths, TEXT("\n"));
	
	TArray<FInstallInfo> Result;
	for(const FString& RiderPath: RiderPaths)
	{
		TOptional<FInstallInfo> InstallInfo = FRiderPathLocator::GetInstallInfoFromRiderPath(RiderPath, FInstallInfo::EInstallType::Installed);
		if(InstallInfo.IsSet())
		{
			Result.Add(InstallInfo.GetValue());
		}
	}
	return Result;
}

FString FRiderPathLocator::GetDefaultIDEInstallLocationForToolboxV2()
{
#if ENGINE_MAJOR_VERSION == 4 && ENGINE_MINOR_VERSION <= 20
	TCHAR CHomePath[4096];
	FPlatformMisc::GetEnvironmentVariable(TEXT("HOME"), CHomePath, ARRAY_COUNT(CHomePath));
	const FString FHomePath = CHomePath;
#else
	const FString FHomePath = FPlatformMisc::GetEnvironmentVariable(TEXT("HOME"));
#endif
	return FPaths::Combine(FHomePath, TEXT("Applications"));
}

TSet<FInstallInfo> FRiderPathLocator::CollectAllPaths()
{
	TSet<FInstallInfo> InstallInfos;
	InstallInfos.Append(GetInstalledRidersWithMdfind());
	InstallInfos.Append(GetManuallyInstalledRiders());
	InstallInfos.Append(GetInstallInfosFromToolbox(GetToolboxPath(), "Rider*.app"));
	InstallInfos.Append(GetInstallInfosFromResourceFile());
	return InstallInfos;
}
#endif
