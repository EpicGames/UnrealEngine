// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"

struct FVersion
{	
	bool IsInitialized() const { return Versions.Num() != 0; }

	FVersion(){}

	explicit FVersion(const FString& VersionString)
	{
		*this = VersionString;
	}

	void operator=(const FString& VersionString)
	{
		TArray<FString> Output;
		VersionString.ParseIntoArray(Output, TEXT("."));
		for (const FString& Item : Output)
		{
			Versions.Add(FCString::Atoi(*Item));
		}		
	}

	int32 Major() const
	{
		if(Versions.Num() >= 1)
		{
			return Versions[0];
		}
		return INVALID_VERSION;
	}

	int32 Minor() const
	{
		if(Versions.Num() >= 2)
		{
			return Versions[1];
		}
		return INVALID_VERSION;
	}

	int32 Patch() const
	{
		if(Versions.Num() >= 3)
		{
			return Versions[2];
		}
		return INVALID_VERSION;
	}

	bool operator<(const FVersion& rhs) const
	{
		TArray<int32>::SizeType LeftSize = Versions.Num();
		TArray<int32>::SizeType RightSize = rhs.Versions.Num();
		const TArray<int32>::SizeType Size = (LeftSize >= RightSize) ? RightSize : LeftSize;
		for(TArray<int32>::SizeType Index = 0; Index < Size; Index++)
		{
			const int32 LeftElement = Versions[Index];
			const int32 RightElement = rhs.Versions[Index];
			if(LeftElement < RightElement) return true;
			if(LeftElement > RightElement) return false;
		}
		return LeftSize < RightSize;
	}

	bool operator==(const FVersion& rhs) const
	{
		return !(*this < rhs) && !(rhs < *this); 
	}

	bool operator!=(const FVersion& rhs) const
	{
		return (*this < rhs) || (rhs < *this); 
	}

	FString ToString() const
	{
		return FString::JoinBy(Versions, TEXT("."), [](int32 Item)-> FString { return FString::FromInt(Item); });
	}

	
	TArray<int32> Versions;
	static const int32 INVALID_VERSION = -1;
};

struct FInstallInfo
{
	enum class ESupportUproject
	{
		None,
		Beta,
		Release			
	};

	enum class EInstallType
	{
		Installed,
		Toolbox,
		Custom
	};
	
	FString Path;
	FVersion Version;
	ESupportUproject SupportUprojectState = ESupportUproject::None;
	EInstallType InstallType;

	FInstallInfo() = default;

	bool operator<(const FInstallInfo& InstallInfo) const
	{
		return Version < InstallInfo.Version;
	}

	bool operator==(const FInstallInfo& InstallInfo) const
	{
		FString LocalPath = Path;
		FString InputPath = InstallInfo.Path;
		LocalPath.ReplaceInline(TEXT("\\\\"), TEXT("/"), ESearchCase::CaseSensitive);
		FPaths::NormalizeFilename(LocalPath);
		InputPath.ReplaceInline(TEXT("\\\\"), TEXT("/"), ESearchCase::CaseSensitive);
		FPaths::NormalizeFilename(InputPath);
		return !(*this < InstallInfo) && !(InstallInfo < *this) && (LocalPath == InputPath);
	}
    
    friend FORCEINLINE uint32 GetTypeHash(const FInstallInfo& InstallInfo)
    {
		FString LocalPath = InstallInfo.Path;
		LocalPath.ReplaceInline(TEXT("\\\\"), TEXT("/"), ESearchCase::CaseSensitive);
		FPaths::NormalizeFilename(LocalPath);
        return GetTypeHash(LocalPath);
    }
};

template <typename OptionalType> struct TOptional;
class FRiderPathLocator
{
public:
	// Platform specific implementation
	static TOptional<FInstallInfo> GetInstallInfoFromRiderPath(const FString& Path, FInstallInfo::EInstallType InstallType);
	static bool DirectoryExistsAndNonEmpty(const FString& Path);
	static TSet<FInstallInfo> CollectAllPaths();
private:
	static void ParseProductInfoJson(FInstallInfo& Info, const FString& ProductInfoJsonPath);
	static FString GetDefaultIDEInstallLocationForToolboxV2();
	static TArray<FInstallInfo> GetInstallInfosFromToolbox(const FString& ToolboxPath, const FString& Pattern);
	static TArray<FInstallInfo> GetInstallInfosFromResourceFile();
	static TArray<FInstallInfo> GetInstallInfos(const FString& ToolboxRiderRootPath, const FString& Pattern, FInstallInfo::EInstallType InstallType);
	static FString GetHistoryJsonPath(const FString& RiderPath);
	static FVersion GetLastBuildVersion(const FString& HistoryJsonPath);
};
