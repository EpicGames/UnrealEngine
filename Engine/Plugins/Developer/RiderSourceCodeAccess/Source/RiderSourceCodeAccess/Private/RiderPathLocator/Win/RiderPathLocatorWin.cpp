// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
#include "RiderPathLocator/RiderPathLocator.h"

#include "Internationalization/Regex.h"
#include "Misc/Paths.h"

#include "Runtime/Launch/Resources/Version.h"
#include "Windows/WindowsPlatformMisc.h"
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <winreg.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

static FString GetToolboxPath()
{	
	TArray<FInstallInfo> Result;
#if ENGINE_MAJOR_VERSION == 4 && ENGINE_MINOR_VERSION <= 20
	TCHAR CHomePath[4096];
	FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"), CHomePath, ARRAY_COUNT(CHomePath));
	const FString LocalAppData = CHomePath;
#else
	const FString LocalAppData = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
#endif

	return FPaths::Combine(LocalAppData, TEXT("JetBrains"), TEXT("Toolbox"));
}

static FString GetToolboxPath(const Windows::HKEY RootKey, const FString& RegistryKey)
{
	FString ToolboxBinPath;

	if (!FWindowsPlatformMisc::QueryRegKey(RootKey, *RegistryKey, TEXT(""), ToolboxBinPath)) return {};

	FPaths::NormalizeDirectoryName(ToolboxBinPath);
	const FString PatternString(TEXT("(.*)(?:\\\\|/)bin"));
	const FRegexPattern Pattern(PatternString);
	FRegexMatcher ToolboxPathMatcher(Pattern, ToolboxBinPath);
	if (!ToolboxPathMatcher.FindNext()) return {};

	return ToolboxPathMatcher.GetCaptureGroup(1);
}

FString FRiderPathLocator::GetDefaultIDEInstallLocationForToolboxV2()
{
	const FString LocalAppData = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
	return FPaths::Combine(LocalAppData, TEXT("Programs"));
}

static bool EnumerateRegistryKeys(HKEY Key, TArray<FString> &OutNames)
{
	for (DWORD Index = 0;; Index++)
	{
		TCHAR KeyName[256];
		DWORD KeyNameLength = sizeof(KeyName) / sizeof(KeyName[0]);

		const LONG Result = RegEnumKeyEx(Key, Index, KeyName, &KeyNameLength, nullptr, nullptr, nullptr, nullptr);
		if (Result == ERROR_NO_MORE_ITEMS)
		{
			break;
		}

		if (Result != ERROR_SUCCESS)
		{
			return false;
		}

		OutNames.Add(KeyName);
	}
	return true;
}

static bool EnumerateRegistryValues(HKEY Key, TArray<FString> &OutNames)
{
	for (DWORD Index = 0;; Index++)
	{
		// Query the value
		wchar_t ValueName[256];
		DWORD ValueNameLength = sizeof(ValueName) / sizeof(ValueName[0]);

		const LONG Result = RegEnumValue(Key, Index, ValueName, &ValueNameLength, nullptr, nullptr, nullptr, nullptr);
		if (Result == ERROR_NO_MORE_ITEMS)
		{
			break;
		}
		
		if (Result != ERROR_SUCCESS)
		{
			return false;
		}

		// Add it to the array
		OutNames.Add(ValueName);
	}
	return true;
}

static LONG GetStringRegKey(const HKEY Key, const FString& ValueName, FString& Value)
{
	WCHAR Buffer[512];
	DWORD BufferSize = sizeof(Buffer);
	const ULONG Result = RegQueryValueExW(Key, *ValueName, nullptr, nullptr, reinterpret_cast<LPBYTE>(Buffer), &BufferSize);
	if (Result == ERROR_SUCCESS)
	{
		Value = Buffer;
	}
	return Result;
}
static TArray<FInstallInfo> CollectPathsFromRegistry( const Windows::HKEY RootKey, const FString& RegistryKey)
{
	TArray<FInstallInfo> InstallInfos;
	HKEY Key;
	const LONG Result = RegOpenKeyEx(RootKey, *RegistryKey, 0, KEY_READ, &Key);
	if (Result != ERROR_SUCCESS) return InstallInfos;
	
	TArray<FString> Keys;
	if (!EnumerateRegistryKeys(Key, Keys)) return InstallInfos;
	
	for (const FString& key : Keys)
	{
		if (!key.Contains(TEXT("Rider"))) continue;

		HKEY SubKey;
		const LONG SubResult = RegOpenKeyEx(Key, *key, 0, KEY_READ, &SubKey);
		if (SubResult != ERROR_SUCCESS) continue;

		TArray<FString> Values;
		if (!EnumerateRegistryValues(SubKey, Values)) continue;

		for (const auto& Value : Values)
		{
			if (Value != TEXT("InstallLocation")) continue;
			FString InstallLocation;
			if (GetStringRegKey(SubKey, Value, InstallLocation) != ERROR_SUCCESS) continue;
			
			const FString ExePath = FPaths::Combine(InstallLocation, TEXT("bin"), TEXT("rider64.exe"));
			TOptional<FInstallInfo> InstallInfo = FRiderPathLocator::GetInstallInfoFromRiderPath(ExePath, FInstallInfo::EInstallType::Installed);
			if(InstallInfo.IsSet())
			{
				InstallInfos.Add(InstallInfo.GetValue());
			}
		}
	}

	return InstallInfos;
}

static TArray<FInstallInfo> CollectDotUltimatePathsFromRegistry( const Windows::HKEY RootKey, const FString& RegistryKey)
{
	TArray<FInstallInfo> InstallInfos;
	HKEY Key;
	const LONG Result = RegOpenKeyEx(RootKey, *RegistryKey, 0, KEY_READ, &Key);
	if (Result != ERROR_SUCCESS) return InstallInfos;
	
	TArray<FString> Keys;
	if (!EnumerateRegistryKeys(Key, Keys)) return InstallInfos;
	
	for (const FString& key : Keys)
	{
		HKEY SubKey;
		const LONG SubResult = RegOpenKeyEx(Key, *key, 0, KEY_READ, &SubKey);
		if (SubResult != ERROR_SUCCESS) continue;

		TArray<FString> Values;
		if (!EnumerateRegistryValues(SubKey, Values)) continue;

		for (const auto& Value : Values)
		{
			if (Value != TEXT("InstallDir")) continue;
			
			FString InstallLocation;
			if (GetStringRegKey(SubKey, Value, InstallLocation) != ERROR_SUCCESS) continue;
			
			const FString ExePath = FPaths::Combine(InstallLocation, TEXT("bin"), TEXT("rider64.exe"));
			TOptional<FInstallInfo> InstallInfo = FRiderPathLocator::GetInstallInfoFromRiderPath(ExePath, FInstallInfo::EInstallType::Installed);
			if(InstallInfo.IsSet())
			{
				InstallInfos.Add(InstallInfo.GetValue());
			}
		}
	}

	return InstallInfos;
}

TOptional<FInstallInfo> FRiderPathLocator::GetInstallInfoFromRiderPath(const FString& Path, FInstallInfo::EInstallType InstallType)
{
	if(!FPaths::FileExists(Path))
	{
		return {};
	}
	
	const FString PatternString(TEXT("(.*)(?:\\\\|/)bin"));
	const FRegexPattern Pattern(PatternString);
	FRegexMatcher RiderPathMatcher(Pattern, Path);
	if (!RiderPathMatcher.FindNext())
	{
		return {};
	}

	const FString RiderDir = RiderPathMatcher.GetCaptureGroup(1);
	const FString RiderCppPluginPath = FPaths::Combine(RiderDir, TEXT("plugins"), TEXT("rider-cpp"));
	if (!DirectoryExistsAndNonEmpty(RiderCppPluginPath))
	{
		return {};
	}
	
	FInstallInfo Info;
	Info.Path = Path;
	Info.InstallType = InstallType;
	const FString ProductInfoJsonPath = FPaths::Combine(RiderDir, TEXT("product-info.json"));
	if (FPaths::FileExists(ProductInfoJsonPath))
	{
		ParseProductInfoJson(Info, ProductInfoJsonPath);
	}
	if(!Info.Version.IsInitialized())
	{
		Info.Version = FPaths::GetBaseFilename(RiderDir);
		if(Info.Version.Major() >= 221)
		{
			Info.SupportUprojectState = FInstallInfo::ESupportUproject::Release;
		}
	}
	return Info;
}

TSet<FInstallInfo> FRiderPathLocator::CollectAllPaths()
{
	TSet<FInstallInfo> InstallInfos;
	InstallInfos.Append(CollectPathsFromRegistry(HKEY_CURRENT_USER, TEXT("SOFTWARE\\Microsoft\\Windows\\CurrentVersion\\Uninstall")));
	InstallInfos.Append(CollectPathsFromRegistry(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Microsoft\\Windows\\CurrentVersion\\Uninstall")));
	InstallInfos.Append(CollectPathsFromRegistry(HKEY_CURRENT_USER, TEXT("SOFTWARE\\WOW6432Node\\Microsoft\\Windows\\CurrentVersion\\Uninstall")));
	InstallInfos.Append(CollectPathsFromRegistry(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\WOW6432Node\\Microsoft\\Windows\\CurrentVersion\\Uninstall")));
	InstallInfos.Append(CollectDotUltimatePathsFromRegistry(HKEY_CURRENT_USER, TEXT("SOFTWARE\\JetBrains\\Rider")));
	InstallInfos.Append(CollectDotUltimatePathsFromRegistry(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\JetBrains\\Rider")));
	InstallInfos.Append(CollectDotUltimatePathsFromRegistry(HKEY_CURRENT_USER, TEXT("SOFTWARE\\WOW6432Node\\JetBrains\\Rider")));
	InstallInfos.Append(CollectDotUltimatePathsFromRegistry(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\WOW6432Node\\JetBrains\\Rider")));
	InstallInfos.Append(GetInstallInfosFromToolbox(GetToolboxPath(), "rider64.exe"));
	InstallInfos.Append(GetInstallInfosFromToolbox(GetToolboxPath(HKEY_CURRENT_USER, TEXT("Software\\JetBrains\\Toolbox\\")), "rider64.exe"));
	InstallInfos.Append(GetInstallInfosFromToolbox(GetToolboxPath(HKEY_LOCAL_MACHINE, TEXT("Software\\JetBrains\\Toolbox\\")), "rider64.exe"));
	InstallInfos.Append(GetInstallInfosFromToolbox(GetToolboxPath(HKEY_CURRENT_USER, TEXT("Software\\JetBrains s.r.o.\\JetBrainsToolbox\\")), "rider64.exe"));
	InstallInfos.Append(GetInstallInfosFromToolbox(GetToolboxPath(HKEY_LOCAL_MACHINE, TEXT("Software\\JetBrains s.r.o.\\JetBrainsToolbox\\")), "rider64.exe"));
	InstallInfos.Append(GetInstallInfosFromResourceFile());
	return InstallInfos;
}
#endif