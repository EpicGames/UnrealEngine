// Copyright Epic Games, Inc. All Rights Reserved.

#include "RiderSourceCodeAccessor.h"

#include "RiderPathLocator/RiderPathLocator.h"

#include "Modules/ModuleManager.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/UProjectInfo.h"
#include "DesktopPlatformModule.h"
#include "ISourceCodeAccessModule.h"
#include "Interfaces/IProjectManager.h"
#include "ProjectDescriptor.h"
#include "GameProjectGenerationModule.h"
#include "Dialogs/SOutputLogDialog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/MessageDialog.h"
#include "Style/RiderSourceCodeAccessStyleSet.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "RiderSourceCodeAccessor"

DEFINE_LOG_CATEGORY_STATIC(LogRiderAccessor, Log, All);

namespace RSCA
{
TOptional<FString> ResolvePathToFile(const FString& FullPath)
{
	FString Path = FullPath;
	if (FPaths::IsRelative(Path))
		Path = FPaths::ConvertRelativePathToFull(Path);

	if (!FPaths::FileExists(Path))
	{
		static const TArray<FString> SubDirs = {
			"/Engine/Source/",
			"/Engine/Plugins/"
		};

		FString EngineRootDir = FPaths::RootDir();
		FPaths::NormalizeFilename(Path);
		int32 Index = INDEX_NONE;
		for (const FString& SubDir : SubDirs)
		{
			Index = Path.Find(SubDir);
			if (Index != INDEX_NONE) break;
		}
		if (Index == INDEX_NONE) return {};

		Path = EngineRootDir.Append(Path.RightChop(Index));
		if (!FPaths::FileExists(Path)) return {};
	}
	return {Path};
}

struct FCommandLineInfo
{
	FString App;
	FString Args;
};

#if PLATFORM_MAC
int processIsTranslated() {
	int ret = 0;
	size_t size = sizeof(ret);
	if (sysctlbyname("sysctl.proc_translated", &ret, &size, NULL, 0) == -1) 
	{
		if (errno == ENOENT)
			return 0;
		return -1;
	}
	return ret;
}
#endif

FCommandLineInfo GetPlatformAppAndArgs(const FString& App, const FString& Args)
{
	FCommandLineInfo info;
	info.App = App;
    info.Args = Args;
#if PLATFORM_MAC
	if(processIsTranslated() == 1)
	{
		info.App = TEXT("/usr/bin/arch");
		info.Args = FString::Printf(TEXT("-arm64 \"%s\" %s"), *App, *Args);
	}
#endif
	return info;
}

bool CheckExecutable(const FString& App)
{
	if(FPaths::FileExists(App) || FPaths::DirectoryExists(App))
	{
		return true;
	}

	FNotificationInfo Info( FText::Format(LOCTEXT("CodeAccessorAppDoesntExist", "{0} doesn't exist"), FText::FromString(App)) );
	Info.bFireAndForget = true;

	FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);

	return false;
}

bool OpenRider(const FString& ExecutablePath, const FString& Params, const FString& ErrorMessage)
{	
	const FCommandLineInfo PlatformAppAndArgs = GetPlatformAppAndArgs(ExecutablePath, Params);
	if(!CheckExecutable(PlatformAppAndArgs.App))
	{
		return false;
	}
	FProcHandle Proc = FPlatformProcess::CreateProc(*PlatformAppAndArgs.App, *PlatformAppAndArgs.Args, true, true, false, nullptr, 0,
													nullptr, nullptr);
	const bool bResult = Proc.IsValid();
	if (!bResult)
	{
		UE_LOGF(LogRiderAccessor, Warning, "%ls", *ErrorMessage);
		FPlatformProcess::CloseProc(Proc);
	}
	return bResult;
}

}

void FRiderSourceCodeAccessor::RefreshAvailability()
{
	// If we have an executable path, we certainly have it installed!
	bHasRiderInstalled = !ExecutablePath.IsEmpty() && FPaths::FileExists(ExecutablePath);
}

bool FRiderSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	// For uproject model, we're listening to changes of filesystem and will update project automatically
	if(Model == EProjectModel::Uproject) return true;
	
	// For other cases, fall back to default one
	return false;
}

bool FRiderSourceCodeAccessor::CanAccessSourceCode() const
{
	return bHasRiderInstalled;
}

bool FRiderSourceCodeAccessor::DoesSolutionExist() const
{
	CachePathToSolution();
	return FPaths::FileExists(CachedSolutionPath);
}

FText FRiderSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("RiderDisplayDesc", "Open source code files in Rider");
}

FName FRiderSourceCodeAccessor::GetFName() const
{
	return RiderName;
}

FText FRiderSourceCodeAccessor::GetNameText() const
{
	return FText::FromName(RiderName);
}

bool FRiderSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32)
{
	if (!bHasRiderInstalled) return false;
	TOptional<FString> OptionalSolutionPath = GetSolutionPath();
	if (!OptionalSolutionPath.IsSet()) return false;
	
	FString SolutionPath = OptionalSolutionPath.GetValue();
	if (FPaths::IsRelative(SolutionPath))
		SolutionPath = FPaths::ConvertRelativePathToFull(SolutionPath);

	const TOptional<FString> OptionalPath = RSCA::ResolvePathToFile(FullPath);
	if(!OptionalPath.IsSet()) return false;
	
	const FString Path = OptionalPath.GetValue();
	const FString Params = FString::Printf(TEXT("\"%s\" --line %d \"%s\""), *SolutionPath, LineNumber, *Path);
	const FString ErrorMessage = FString::Printf(TEXT("Opening file (%s) at a line (%d) failed."), *Path, LineNumber);

	return HandleOpeningRider([this, &Params, &ErrorMessage]() -> bool
	{
		return RSCA::OpenRider(ExecutablePath, Params, ErrorMessage);
	});
}

bool FRiderSourceCodeAccessor::OpenSolution()
{
	if (!bHasRiderInstalled) return false;
	
	TOptional<FString> OptionalSolutionPath = GetSolutionPath();
	if (!OptionalSolutionPath.IsSet()) return false;

	const FString SolutionPath = OptionalSolutionPath.GetValue();
	
	const FString FullPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*SolutionPath);
	const FString Params = FString::Printf(TEXT("\"%s\""), *FullPath);
	const FString ErrorMessage = FString::Printf(TEXT("Opening solution (%s) failed."), *FullPath);

	return HandleOpeningRider([this, &Params, &ErrorMessage]()->bool
	{		
		return RSCA::OpenRider(ExecutablePath, Params, ErrorMessage);
	});
}
bool FRiderSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	if (!bHasRiderInstalled) return false;

	FString CorrectSolutionPath = InSolutionPath;
	if (!CorrectSolutionPath.EndsWith(".sln"))
	{
		CorrectSolutionPath += ".sln";
	}
	const FString Params = FString::Printf(TEXT("\"%s\""), *CorrectSolutionPath);
	const FString ErrorMessage = FString::Printf(TEXT("Opening the project file (%s) failed."), *CorrectSolutionPath);

	return HandleOpeningRider([this, &Params, &ErrorMessage]()->bool
	{		
		return RSCA::OpenRider(ExecutablePath, Params, ErrorMessage);
	});
}

bool FRiderSourceCodeAccessor::HandleOpeningRider(const TFunction<bool()> Callback) const
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>(TEXT("SourceCodeAccess"));
	SourceCodeAccessModule.OnLaunchingCodeAccessor().Broadcast();
	const bool bResult = Callback();
	SourceCodeAccessModule.OnDoneLaunchingCodeAccessor().Broadcast(bResult);
	return bResult;
}

bool FRiderSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	if (!bHasRiderInstalled) return false;
	
	TOptional<FString> OptionalSolutionPath = GetSolutionPath();
	if (!OptionalSolutionPath.IsSet()) return false;
	
	FString SolutionPath = OptionalSolutionPath.GetValue();
	
	if (FPaths::IsRelative(SolutionPath))
		SolutionPath = FPaths::ConvertRelativePathToFull(SolutionPath);

	FString FilePaths = "";
	for (const FString & FullPath : AbsoluteSourcePaths) {
		const TOptional<FString> OptionalPath = RSCA::ResolvePathToFile(FullPath);
		if(!OptionalPath.IsSet()) return false;
		const FString Path = OptionalPath.GetValue();
		FilePaths += FString::Printf(TEXT("\"%s\" "), *Path);
	}

	const FString Params = FString::Printf(TEXT("\"%s\" %s"), *SolutionPath, *FilePaths);
	const FString ErrorMessage = FString::Printf(TEXT("Opening files (%s) failed."), *FilePaths);

	return HandleOpeningRider([this, &Params, &ErrorMessage]()->bool
	{		
		return RSCA::OpenRider(ExecutablePath, Params, ErrorMessage);
	});
}

bool FRiderSourceCodeAccessor::SaveAllOpenDocuments() const
{
	return false;
}

FName FRiderSourceCodeAccessor::GetOpenIconName() const
{
	return FRiderSourceCodeAccessStyleSet::RiderIconName;
}

FName FRiderSourceCodeAccessor::GetRefreshIconName() const
{
	return FRiderSourceCodeAccessStyleSet::RiderRefreshIconName;
}

FName FRiderSourceCodeAccessor::GetStyleSet() const
{
	return FRiderSourceCodeAccessStyleSet::StyleName;
}

void FRiderSourceCodeAccessor::Init(const FInstallInfo& Info, EProjectModel ProjectModel, EAccessType Type)
{
	Model = ProjectModel; 
	ExecutablePath = Info.Path;
	FString SuffixText = "";
	switch (Info.InstallType) {
		case FInstallInfo::EInstallType::Installed: SuffixText = TEXT("(installed)"); break;
		case FInstallInfo::EInstallType::Toolbox: SuffixText = TEXT("(toolbox)");  break;
		case FInstallInfo::EInstallType::Custom: SuffixText = TEXT("(custom)");  break;
		default: ;
	}
	FString UprojectSuffix = "";
	if(ProjectModel == EProjectModel::Uproject)
	{
		UprojectSuffix += " Uproject";
		if(Info.SupportUprojectState == FInstallInfo::ESupportUproject::Beta)
			UprojectSuffix += " (experimental)";
	}
	FString NewName;
	if(Type == EAccessType::Direct)
	{
		NewName = *FString::Format(TEXT("Rider {0} {1}{2}"), { Info.Version.ToString(), SuffixText, UprojectSuffix });
	}
	else
	{
		NewName = *FString::Format(TEXT("Rider{0}"), { UprojectSuffix });
	}

	RiderName = *NewName;
	
	RefreshAvailability();
}


void FRiderSourceCodeAccessor::CachePathToUproject() const
{
	CachedSolutionPath = FPaths::GetProjectFilePath();
}

void FRiderSourceCodeAccessor::CachePathToSln() const
{
	if (CachedSolutionPathOverride.Len() > 0)
	{
		CachedSolutionPath = CachedSolutionPathOverride + TEXT(".sln");
	}
	else
	{
		CachedSolutionPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

		if (!FUProjectDictionary(FPaths::RootDir()).IsForeignProject(CachedSolutionPath))
		{
			FString PrimaryProjectName;
			if (!FFileHelper::LoadFileToString(PrimaryProjectName, *(FPaths::EngineIntermediateDir() / TEXT("ProjectFiles/PrimaryProjectName.txt"))))
			{
				PrimaryProjectName = "UE5";
			}
			CachedSolutionPath = FPaths::Combine(FPaths::RootDir(), PrimaryProjectName + TEXT(".sln"));
		}
		else
		{
			const FProjectDescriptor* CurrentProject = IProjectManager::Get().GetCurrentProject();

			if (CurrentProject == nullptr || CurrentProject->Modules.Num() == 0)
			{
				CachedSolutionPath = TEXT("");
			}
			else
			{
				const FString BaseName = FApp::HasProjectName() ? FApp::GetProjectName() : FPaths::GetBaseFilename(CachedSolutionPath);
				CachedSolutionPath = FPaths::Combine(CachedSolutionPath, BaseName + TEXT(".sln"));
			}
		}
	}
}

void FRiderSourceCodeAccessor::CachePathToSolution() const
{
	if(IsInGameThread())
	{
		if (Model == EProjectModel::Sln)
		{
			CachePathToSln();
		}			
		else if (Model == EProjectModel::Uproject)
		{
			CachePathToUproject();
		}			
	}
}

bool FRiderSourceCodeAccessor::TryGenerateSlnFile() const
{
#if WITH_EDITOR
	const FText Message = LOCTEXT("RSCA_AskGenerateSolutionFile", "Project file is not available.\nGenerate project file?");
	if (FMessageDialog::Open(EAppMsgType::YesNo, Message) == EAppReturnType::No)
	{
		return false;
	}
	
	FText FailReason, FailLog;
	if(!FGameProjectGenerationModule::Get().UpdateCodeProject(FailReason, FailLog))
	{
		SOutputLogDialog::Open(LOCTEXT("RSCA_GenerateSolutionFile", "Generating Project"), FailReason, FailLog, FText::GetEmpty());
		return false;
	}
	return true;
#else
	return false;
#endif
}

bool FRiderSourceCodeAccessor::TryGenerateSolutionFile() const
{
	if(Model == EProjectModel::Sln)
		return TryGenerateSlnFile();
	// {Game}.uproject should be always available, and Rider project model will be generated on opening/changing project related files 
	if(Model == EProjectModel::Uproject)
		return true;
	return false;
}

TOptional<FString> FRiderSourceCodeAccessor::GetSolutionPath() const
{
	FScopeLock Lock(&CachedSolutionPathCriticalSection);

	CachePathToSolution();

	if(!FPaths::FileExists(CachedSolutionPath))
	{
		if(!TryGenerateSolutionFile()) return {};
		CachePathToSolution();
		if(!FPaths::FileExists(CachedSolutionPath)) return {};
	}

	// This must be an absolute path as VS always uses absolute paths
	return CachedSolutionPath;
}

#undef LOCTEXT_NAMESPACE
