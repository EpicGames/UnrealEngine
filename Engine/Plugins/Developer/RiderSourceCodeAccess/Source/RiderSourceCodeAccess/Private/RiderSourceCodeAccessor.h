// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceCodeAccessor.h"

template <typename OptionalType> struct TOptional;

struct FInstallInfo;

class FRiderSourceCodeAccessor : public ISourceCodeAccessor
{
public:

	static FName FeatureType() {return TEXT("SourceCodeAccessor");};
	
	enum class EAccessType
	{
		Direct,
		Aggregate
	};

	enum class EProjectModel
	{
		Sln,
		Uproject
	};
	
	void Init(const FInstallInfo& Info, EProjectModel ProjectModel, EAccessType Type = EAccessType::Direct);

	/** ISourceCodeAccessor implementation */
	virtual void RefreshAvailability() override;
	virtual bool CanAccessSourceCode() const override;
	virtual bool DoesSolutionExist() const override;
	virtual FName GetFName() const override;
	virtual FText GetNameText() const override;
	virtual FText GetDescriptionText() const override;
	virtual bool OpenSolution() override;
	virtual bool OpenSolutionAtPath(const FString& InSolutionPath) override;
	virtual bool OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0) override;
	virtual bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) override;
	virtual bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules) override;
	virtual bool SaveAllOpenDocuments() const override;
	virtual void Tick(const float) override {}
	virtual FName GetOpenIconName() const override;
	virtual FName GetRefreshIconName() const override;
	virtual FName GetStyleSet() const override;
private:
	void CachePathToUproject() const;
	void CachePathToSln() const;
	void CachePathToSolution() const;
	bool TryGenerateSlnFile() const;
	bool HandleOpeningRider(TFunction<bool()> Callback) const;

	bool TryGenerateSolutionFile() const;
	TOptional<FString> GetSolutionPath() const;

	FName RiderName;

	/** Is Rider installed on this system? */
	bool bHasRiderInstalled = false;
	
	/** The path to the Rider executable. */
	FString ExecutablePath;

	/** Critical section for updating SolutionPath */
	mutable FCriticalSection CachedSolutionPathCriticalSection;

	/** String storing the solution path obtained from the module manager to avoid having to use it on a thread */
	mutable FString CachedSolutionPath = {};

	/** Override for the cached solution path */
	mutable FString CachedSolutionPathOverride = {};
	EProjectModel Model = EProjectModel::Sln;
};
