// Copyright Epic Games, Inc. All Rights Reserved.

#include "RiderSourceCodeAccessorModule.h"

#include "RiderPathLocator/RiderPathLocator.h"
#include "RiderSourceCodeAccessor.h"

#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "Style/RiderSourceCodeAccessStyleSet.h"

#define LOCTEXT_NAMESPACE "RiderSourceCodeAccessor"

IMPLEMENT_MODULE(FRiderSourceCodeAccessModule, RiderSourceCodeAccess);

void FRiderSourceCodeAccessModule::StartupModule()
{
	FRiderSourceCodeAccessStyleSet::Initialize();
	
	TArray<FInstallInfo> InstallInfos = FRiderPathLocator::CollectAllPaths().Array();
	InstallInfos.Sort();
	GenerateUprojectAccessors(InstallInfos);
	GenerateSlnAccessors(InstallInfos);
}

bool FRiderSourceCodeAccessModule::SupportsDynamicReloading()
{
	return true;
}

void FRiderSourceCodeAccessModule::ShutdownModule()
{
	for (auto& RiderSourceCodeAccessor : RiderSourceCodeAccessors)
	{
		// Unbind provider from editor
		IModularFeatures::Get().UnregisterModularFeature(FRiderSourceCodeAccessor::FeatureType(), &(RiderSourceCodeAccessor.Value.Get()));
	}
	
	FRiderSourceCodeAccessStyleSet::Shutdown();
}

void FRiderSourceCodeAccessModule::GenerateSlnAccessors(const TArray<FInstallInfo>& InstallInfos)
{
#if PLATFORM_WINDOWS
	if(InstallInfos.Num() == 0) return;
	
	if(InstallInfos.Num() > 1)
	{
		for (const FInstallInfo& InstallInfo : InstallInfos)
		{
			TSharedRef<FRiderSourceCodeAccessor> RiderSourceCodeAccessor = MakeShareable(new FRiderSourceCodeAccessor());
			RiderSourceCodeAccessor->Init(InstallInfo, FRiderSourceCodeAccessor::EProjectModel::Sln);
			IModularFeatures::Get().RegisterModularFeature(FRiderSourceCodeAccessor::FeatureType(), &RiderSourceCodeAccessor.Get());
			RiderSourceCodeAccessors.Add(RiderSourceCodeAccessor->GetFName(), RiderSourceCodeAccessor);
		}

	}

	const TSharedRef<FRiderSourceCodeAccessor> RiderSourceCodeAccessor = MakeShareable(new FRiderSourceCodeAccessor());
	RiderSourceCodeAccessor->Init(InstallInfos.Last(), FRiderSourceCodeAccessor::EProjectModel::Sln,
	                              FRiderSourceCodeAccessor::EAccessType::Aggregate);
	IModularFeatures::Get().RegisterModularFeature(FRiderSourceCodeAccessor::FeatureType(), &RiderSourceCodeAccessor.Get());
	RiderSourceCodeAccessors.Add(TEXT("Rider"), RiderSourceCodeAccessor);
#endif
}

void FRiderSourceCodeAccessModule::GenerateUprojectAccessors(const TArray<FInstallInfo>& InstallInfos)
{
	TArray<TArray<FInstallInfo>::ElementType> UprojectInfos = InstallInfos.FilterByPredicate([](const FInstallInfo& Item) -> bool
	{
		return Item.SupportUprojectState != FInstallInfo::ESupportUproject::None;
	});

	if(UprojectInfos.Num() == 0) return;

	if(UprojectInfos.Num() > 1)
	{
		for (const FInstallInfo& UprojectInfo : UprojectInfos)
		{
			TSharedRef<FRiderSourceCodeAccessor> RiderSourceCodeAccessor = MakeShareable(new FRiderSourceCodeAccessor());
			RiderSourceCodeAccessor->Init(UprojectInfo, FRiderSourceCodeAccessor::EProjectModel::Uproject);
			IModularFeatures::Get().RegisterModularFeature(FRiderSourceCodeAccessor::FeatureType(), &RiderSourceCodeAccessor.Get());
			RiderSourceCodeAccessors.Add(RiderSourceCodeAccessor->GetFName(), RiderSourceCodeAccessor);
		}
	}

	const TSharedRef<FRiderSourceCodeAccessor> RiderSourceCodeAccessor = MakeShareable(new FRiderSourceCodeAccessor());
	RiderSourceCodeAccessor->Init(InstallInfos.Last(), FRiderSourceCodeAccessor::EProjectModel::Uproject,
                                  FRiderSourceCodeAccessor::EAccessType::Aggregate);
	IModularFeatures::Get().RegisterModularFeature(FRiderSourceCodeAccessor::FeatureType(), &RiderSourceCodeAccessor.Get());
	FString AccessorName = TEXT("Rider uproject");
	if(InstallInfos.Last().SupportUprojectState == FInstallInfo::ESupportUproject::Beta)
		AccessorName += TEXT(" (experimental)");
	
	RiderSourceCodeAccessors.Add(*AccessorName, RiderSourceCodeAccessor);
}

#undef LOCTEXT_NAMESPACE
