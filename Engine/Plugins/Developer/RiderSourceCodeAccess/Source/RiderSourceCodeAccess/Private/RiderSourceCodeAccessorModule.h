// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceCodeAccessModule.h"

class FRiderSourceCodeAccessModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;
private:
	void GenerateSlnAccessors(const TArray<struct FInstallInfo>& InstallInfos);
	void GenerateUprojectAccessors(const TArray<struct FInstallInfo>& InstallInfos);
	TMultiMap<FName, TSharedRef<ISourceCodeAccessor>> RiderSourceCodeAccessors;
};
