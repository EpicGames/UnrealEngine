// Copyright Epic Games, Inc. All Rights Reserved.

#include "RiderSourceCodeAccessStyleSet.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"

FName FRiderSourceCodeAccessStyleSet::RiderRefreshIconName("RSCA.RiderRefreshIcon");
FName FRiderSourceCodeAccessStyleSet::RiderIconName("RSCA.RiderIcon");
FName FRiderSourceCodeAccessStyleSet::StyleName("RiderSourceCodeAccessStyle");
TUniquePtr<FRiderSourceCodeAccessStyleSet> FRiderSourceCodeAccessStyleSet::Inst(nullptr);


const FRiderSourceCodeAccessStyleSet& FRiderSourceCodeAccessStyleSet::Get()
{
	ensure(Inst.IsValid());
	return *(Inst.Get());
}

void FRiderSourceCodeAccessStyleSet::Initialize()
{
	if (!Inst.IsValid())
	{
		Inst = TUniquePtr<FRiderSourceCodeAccessStyleSet>(new FRiderSourceCodeAccessStyleSet);
	}
}

void FRiderSourceCodeAccessStyleSet::Shutdown()
{
	if (Inst.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*Inst.Get());
		Inst.Reset();
	}
}

FRiderSourceCodeAccessStyleSet::FRiderSourceCodeAccessStyleSet() : FSlateStyleSet(StyleName)
{
	SetParentStyleName(FAppStyle::GetAppStyleSetName());
	FString Path = IPluginManager::Get().FindPlugin(TEXT("RiderSourceCodeAccess"))->GetBaseDir() / TEXT("Resources");
	FSlateStyleSet::SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("RiderSourceCodeAccess"))->GetBaseDir() / TEXT("Resources"));

	Set(RiderRefreshIconName, new FSlateVectorImageBrush(FSlateStyleSet::RootToContentDir(TEXT("RiderRefresh.svg")), CoreStyleConstants::Icon16x16));
	Set(RiderIconName, new FSlateVectorImageBrush(FSlateStyleSet::RootToContentDir(TEXT("Rider.svg")), CoreStyleConstants::Icon16x16));
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);	
}

const FName& FRiderSourceCodeAccessStyleSet::GetStyleSetName() const
{
	return StyleName;
}

