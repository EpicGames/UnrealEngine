// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FRiderSourceCodeAccessStyleSet : public FSlateStyleSet
{
public:

	static const FRiderSourceCodeAccessStyleSet& Get();

	static void Initialize();

	static void Shutdown();

	static FName RiderIconName;
	static FName RiderRefreshIconName;
	static FName StyleName;

private:
	FRiderSourceCodeAccessStyleSet();

	virtual const FName& GetStyleSetName() const override;

	static TUniquePtr<FRiderSourceCodeAccessStyleSet> Inst;
	
};
