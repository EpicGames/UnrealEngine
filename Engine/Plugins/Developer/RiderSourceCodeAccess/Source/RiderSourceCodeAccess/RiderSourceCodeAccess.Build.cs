// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RiderSourceCodeAccess : ModuleRules
	{
        public RiderSourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
        {
	        PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		    PrivateDependencyModuleNames.AddRange(
                new []
                {
                    "Core",
                    "SourceCodeAccess",
                    "DesktopPlatform",
					"Projects",
					"Json",
					"Projects",
					"Slate",
					"SlateCore"
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				#if UE_5_0_OR_LATER
				PrivateDependencyModuleNames.Add("EditorFramework");
				#endif
				PrivateDependencyModuleNames.Add("UnrealEd");
				PrivateDependencyModuleNames.Add("GameProjectGeneration");
			}
		}
	}
}